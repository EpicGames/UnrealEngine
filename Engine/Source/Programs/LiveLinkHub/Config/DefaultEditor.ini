[EditorSettings]
; We don't allow creating new plugins.
bCanCreatePluginsFromBrowser=false

; We allow modifying user plugin paths...
bCanModifyUserPluginDirectoriesFromBrowser=true
; ...but not paths stored in the .uproject descriptor.
bCanModifyProjectPluginDirectoriesFromBrowser=false

; Hide plugins that aren't "Installed" or "External" to cut down on noise.
+HidePluginCategoriesFromBrowser=All
+HidePluginCategoriesFromBrowser=Built-In


[Analytics]
AppIdOverride=LiveLinkHub

[/Script/UnrealEd.LevelEditorViewportSettings]
bLevelEditorJoystickControls=False
