[/Script/UnrealEd.EditorLoadingSavingSettings]
bAutoSaveEnable=False
