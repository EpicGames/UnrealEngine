[SectionsToSave]
bCanSaveAllSections=true

[LiveLink]
bUseLiveLinkHubSettingsDetailCustomization=true
bPanelControllerSeparateSourcesSubjects=false
bUseModularClientReference=true
bEnableParentSubjects=true
bCreateLiveLinkHubInstance=true
bEnableReconnectingToStaleSource=true

[LiveLinkHub]
bEnableLLHMessageBusSourceFactory=true
bUseMessagingConnectionManager=true

[TimedDataMonitor]
bEnableTimedDataMonitor=true

[/Script/MacTargetPlatform.XcodeProjectSettings]
TemplateMacPlist=(FilePath="/Engine/Source/Programs/LiveLinkHub/Resources/Mac/Info.plist")

[/Script/LiveLinkInterface.LiveLinkDefaultSourceSettings]
DefaultSourceFrameBufferSize=1

[/Script/LiveLinkInterface.LiveLinkDefaultSubjectSettings]
bRebroadcastSubjectsByDefault=true
bAllowEditingRebroadcastProperty=false

[/Script/LiveLinkHub.LiveLinkHubSettings]
AutoConnectClients=LocalOnly
+ProjectSettingsToDisplay=Live Link
+ProjectSettingsToDisplay=Live Link Hub
+ProjectSettingsToDisplay=Timing & Sync
+ProjectSettingsToDisplay=Messaging
+ProjectSettingsToDisplay=UDPMessaging
bShowFrameRate=True
TargetFrameRate=60.000000

[/Script/AppleARKit.AppleARKitSettings]
bEnableLiveLinkForFaceTracking=false

[/Script/UnrealEd.ProjectPackagingSettings]
bShareMaterialShaderCode=false

[ConsoleVariables]
DetailsPanel.Style.EnableCardLayout=true
Slate.EnableGamepadEditorNavigation=false
LiveLinkHub.EnableStatusBar=false

[/Script/BuildSettings.BuildSettings]
DefaultEditorTarget=LiveLinkHub
; Disabling this to ensure that DesktopPlatformBase doesn't generate a TargetInfo.json file with UE's hundreds of targets
bIncludeParentAssembly=false

[/Script/InterchangeEngine.InterchangeProjectSettings]
ContentImportSettings=()
SceneImportSettings=()
FilePickerClass=None
bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing=True
GenericPipelineClass=None
ConverterDefaultPipeline=None

; Use the engine fallback codecs since engine plugins are disabled
[Animation.DefaultObjectSettings]
BoneCompressionSettings="/Engine/Animation/DefaultAnimBoneCompressionSettings"
CurveCompressionSettings="/Engine/Animation/DefaultAnimCurveCompressionSettings"

[/Script/UdpMessaging.UdpMessagingSettings]
NodeMemoryLoggingThresholdMB=5

[CoreRedirects]
+PropertyRedirects=(OldName="LiveLinkHubSettings.bAutoAddDiscoveredClients",NewName="AutoConnectClients")
+ClassRedirects=(OldName="/Script/LiveLinkHub.LiveLinkHubSubjectSettings",NewName="/Script/LiveLink.LiveLinkHubSubjectSettings")

[MediaProfile]
ButtonMenu="AssetEditor.LiveLinkHub.ToolBar"
ButtonAlignment="Last"
bUseTransientProfile=true
bAllowActiveViewportCapture=false
bAllowManagedViewportCapture=false