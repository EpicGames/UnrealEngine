[CookedEditorSettings_CookedEditor]
CookedEditorTargetName=LiveLinkHub
bStageShaderDirs=true
bStagePlatformBuildDirs=false
bStageExtrasDirs=false
bStagePlatformDirs=false
bStageUAT=false
bStagePython=false
bIsForExternalDistribution=true


[/Script/LiveLink.LiveLinkSettings]
DefaultSettingsClass=/Script/LiveLink.LiveLinkHubSubjectSettings
-DefaultRoleSettings=(Role="/Script/CoreUObject.Class'/Script/LiveLinkInterface.LiveLinkAnimationRole'",SettingClass="/Script/CoreUObject.Class'/Script/LiveLinkInterface.LiveLinkSubjectSettings'",FrameInterpolationProcessor="/Script/CoreUObject.Class'/Script/LiveLink.LiveLinkAnimationFrameInterpolationProcessor'",FramePreProcessors=)
+DefaultRoleSettings=(Role="/Script/CoreUObject.Class'/Script/LiveLinkInterface.LiveLinkAnimationRole'",SettingClass="/Script/CoreUObject.Class'/Script/LiveLink.LiveLinkHubSubjectSettings'",FrameInterpolationProcessor="/Script/CoreUObject.Class'/Script/LiveLink.LiveLinkAnimationFrameInterpolationProcessor'",FramePreProcessors=)
DefaultVirtualSubjectClass=/Script/LiveLinkHub.LiveLinkHubAnimationVirtualSubject
bPreProcessRebroadcastFrames=true
bTranslateRebroadcastFrames=true

[/Script/EngineSettings.GeneralProjectSettings]
ProjectID=914C155297C9471A9E34100517697A75
ProjectName=LiveLinkHub

[/Script/MediaFrameworkUtilities.MediaProfileSettings]
bApplyInCommandlet=False
bAutosaveAfterChange=True
StartupMediaProfile=None

