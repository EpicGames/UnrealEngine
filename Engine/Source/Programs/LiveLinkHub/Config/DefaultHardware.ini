[GPU_NVIDIA Windows]
SuggestedDriverVersion="581.08"
+DriverDenyList=(DriverVersion="<581.08", Reason="These driver versions have a known stability issue that causes a memory leak in Live Link Hub.", RHI="D3D12")