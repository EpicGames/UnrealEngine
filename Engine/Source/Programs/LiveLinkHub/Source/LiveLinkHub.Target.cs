// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;


[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class LiveLinkHubTarget : TargetRules
{
	// Restrict OptedInModulePlatforms to the current Target.Platform.
	// Used during staging, which otherwise fails in TargetPlatform-related restricted
	// subdirectories (Engine/Binaries/Win64/{Android,IOS,Linux,LinuxArm64,...}).
	[CommandLine("-SingleModulePlatform")]
	public bool bSingleModulePlatform = false;

	// Whether to enable building MetaHuman LiveLink plugin.
	[CommandLine("-EnableMetaHumanLiveLinkPlugin=")]
	public bool bEnableMetaHumanLiveLinkPlugin = true;

	public LiveLinkHubTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Editor;
		bExplicitTargetForType = true;
		bGenerateProgramProject = true;

		SolutionDirectory = "Programs/LiveLink";
		LaunchModuleName = "LiveLinkHubLauncher";

		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		
		MacPlatform.bForceTargetNameAsExeName = true;
		
		// These plugins are required for running LiveLinkHub. 
		// They may be a direct dependency or a dependency of another plugin.
		EnablePlugins.AddRange(new string[]
		{
			"ACLPlugin",
			"AudioCaptureTimecodeProvider",
			"LiveLink",
			"LiveLinkHub",
			"PluginBrowser",
			"LiveLinkCamera",
			"LiveLinkLens",
			"LensComponent",
			"LiveLinkInputDevice",
			"ContentBrowserAssetDataSource",
			"ProceduralMeshComponent",
			"PropertyAccessEditor",
			"PythonScriptPlugin",
			"UdpMessaging",
			"MessageBusTester",
			"CameraCalibrationCore",
			"AppleARKitFaceSupport",
			"XInputDevice",

			// Live Link Hub Default Devices
			"LiveLinkDevice",
			"LiveLinkUnrealDevice",
			"LiveLinkKiProDevice",
			"LiveLinkSoundDevice",
			"LiveLinkOBSDevice",

			"LiveLinkOpenVR",
			"LiveLinkOpenTrackIO",

			"CaptureManagerApp",
			"CaptureManagerDevices",
			"MediaIOFramework",
			"MediaFrameworkUtilities",
			"AjaMedia",
			"BlackmagicMedia",
			"NDIMedia",
			
		});

		if (bEnableMetaHumanLiveLinkPlugin)
		{
			EnablePlugins.AddRange(new string[]
			{
				"MetaHumanLiveLink",
				"WmfMedia",
				"AvfMedia",
				"StreamingADA"
			});
		}

		if (bSingleModulePlatform)
		{
			// Necessary for staging, but avoided otherwise because it dirties
			// Definitions.CookedEditor.h and triggers rebuilds (incl. UnrealEditor).
			OptedInModulePlatforms = new UnrealTargetPlatform[] { Target.Platform };
		}

		bAllowEnginePluginsEnabledByDefault = false;
		bBuildAdditionalConsoleApp = false;

		// Based loosely on the VCProject.cs logic to construct NMakePath.
		string BaseExeName = "LiveLinkHub";
		OutputFile = "Binaries/" + Platform.ToString() + "/" + BaseExeName;
		if (Configuration != UndecoratedConfiguration)
		{
			OutputFile += "-" + Platform.ToString() + "-" + Configuration.ToString();
		}
		if (Platform == UnrealTargetPlatform.Win64)
		{
			OutputFile += ".exe";
		}

		// Copy the target receipt into the project binaries directory.
		// Prevents "Would you like to build the editor?" prompt on startup when running with project context.
		//
		// TODO?: If instead we moved the .uproject out of Engine/Source/Programs and into Engine/Programs, UBT
		// would correctly(?) pass `-Project=` in the build command. However, in addition to the .target file,
		// this also results in the executable and a _subset_ of the dependent DLLs ending up ending up in the
		// project Binaries directory. So for now, we'll keep splitting the difference like this.
		DirectoryReference ReceiptSrcDir = Unreal.EngineDirectory;
		DirectoryReference ReceiptDestDir = DirectoryReference.Combine(
			Unreal.EngineDirectory, "Source", "Programs", "LiveLinkHub");

		FileReference ReceiptSrcPath = TargetReceipt.GetDefaultPath(ReceiptSrcDir, BaseExeName, Platform, Configuration, Architectures);
		FileReference ReceiptDestPath = TargetReceipt.GetDefaultPath(ReceiptDestDir, BaseExeName, Platform, Configuration, Architectures);
		DirectoryReference.CreateDirectory(ReceiptDestPath.Directory);

		PostBuildSteps.Add($"echo Copying \"{ReceiptSrcPath}\" to \"{ReceiptDestPath}\"");

		if (Platform == UnrealTargetPlatform.Win64)
		{
			PostBuildSteps.Add($"copy /Y \"{ReceiptSrcPath}\" \"{ReceiptDestPath}\"");
		}
		else
		{
			PostBuildSteps.Add($"cp -a \"{ReceiptSrcPath}\" \"{ReceiptDestPath}\"");
		}
	}
}
