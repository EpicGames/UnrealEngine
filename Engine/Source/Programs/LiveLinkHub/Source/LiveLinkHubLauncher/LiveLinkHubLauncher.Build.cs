// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;


// This is named LiveLinkHubLauncher in order to not clash with the LiveLinkHub module which is in the livelink plugin.
public class LiveLinkHubLauncher : ModuleRules
{
	public LiveLinkHubLauncher(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"LiveLinkHub",
				"PluginBrowser",
			});

		// LaunchEngineLoop dependencies
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"Engine",
				"InputCore",
				"InstallBundleManager",
				"MediaUtils",
				"Messaging",
				"MoviePlayer",
				"MoviePlayerProxy",
				"ProfileVisualizer",
				"Projects",
				"PreLoadScreen",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"TraceLog", 
			}
		);

		// LaunchEngineLoop IncludePath dependencies
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"AutomationWorker",
				"AutomationController",
				"AutomationTest",
				"DerivedDataCache",
				"HeadMountedDisplay", 
				"MRMesh", 
				"SlateRHIRenderer", 
				"SlateNullRenderer",
			}
		);

		// LaunchEngineLoop editor dependencies
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"DerivedDataCache",
				"ToolWidgets",
				"UnrealEd"
		});


		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AutomationController",
					"AutomationTest",
					"AutomationWorker",
				});
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.Add("AgilitySDK");
		}

		ShortName = "LLHub";
		bRequiresPlatformSDK = true;

		AddRuntimeDependencies();
	}

	void AddRuntimeDependencies()
	{
		DirectoryReference ModuleDirectoryRef = new DirectoryReference(ModuleDirectory);
		FileReference ConfigLocation = FileReference.Combine(ModuleDirectoryRef, "RuntimeDependencies.ini");
		ConfigHierarchy Config = new ConfigHierarchy(new List<ConfigFile>{ new ConfigFile(ConfigLocation) });

		if (!Config.TryGetValues("LiveLinkHubLauncher", "RuntimeDependencies", out IReadOnlyList<string>? ConfigDependencies))
		{
			string Err = "Error reading runtime dependencies from config";
			Console.WriteLine(Err);
			throw new BuildException(Err);
		}

		FileFilter Filter = new FileFilter();
		foreach (string Rule in ConfigDependencies)
		{
			Filter.AddRule(Rule);
		}

		// Apply the standard exclusion rules (c.f. DeploymentContext ctor)
		HashSet<string> RestrictedFolderNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
		RestrictedFolderNames.UnionWith(PlatformExports.GetPlatformFolderNames());
		RestrictedFolderNames.UnionWith(RestrictedFolder.GetNames());
		RestrictedFolderNames.ExceptWith(PlatformExports.GetIncludedFolderNames(Target.Platform));
		foreach (string RestrictedFolderName in RestrictedFolderNames)
		{
			string Rule = String.Format(".../{0}/...", RestrictedFolderName);
			Filter.AddRule(Rule, FileFilterType.Exclude);
		}

		// Make relative to $(EngineDir) and add to RuntimeDependencies
		List<FileReference> AbsFiles = Filter.ApplyToDirectory(new DirectoryReference(EngineDirectory), true);
		IEnumerable<string> RelFiles = AbsFiles.Select(x => x.FullName.Replace(EngineDirectory, "$(EngineDir)"));
		foreach (string RelFile in RelFiles)
		{
			RuntimeDependencies.Add(RelFile, StagedFileType.UFS);
		}
	}
}
