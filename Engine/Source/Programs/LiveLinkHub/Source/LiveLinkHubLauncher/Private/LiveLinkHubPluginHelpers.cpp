// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubPluginHelpers.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPluginManager.h"
#include "IPluginBrowser.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"


namespace LiveLinkHub::PluginHelpers
{
	const TCHAR* const ConfigSection = TEXT("LiveLinkHub");
	const TCHAR* const ConfigKey = TEXT("PluginDirectories");
}


FString LiveLinkHub::PluginHelpers::GetPluginConfigPath()
{
	// Modeled after FPaths::EngineUserDir()
	if (FPaths::ShouldSaveToUserDir() || FApp::IsEngineInstalled())
	{
		// e.g. %LocalAppData%\UnrealEngine\LiveLinkHub_5.7\Config\LiveLinkHubPlugins.ini
		return FPaths::Combine(
			FPlatformProcess::UserSettingsDir(),
			*FApp::GetEpicProductIdentifier(),
			*FString(TEXT("LiveLinkHub_")) + FEngineVersion::Current().ToString(EVersionComponent::Minor),
			TEXT("Config"),
			TEXT("LiveLinkHubPlugins.ini")
		);
	}
	else
	{
		return FPaths::Combine(
			*FPaths::EngineConfigDir(),
			TEXT("LiveLinkHub"),
			TEXT("LiveLinkHubPlugins.ini")
		);
	}
}

void LiveLinkHub::PluginHelpers::ReadPluginDirectoriesFromConfig(TArray<FString>& OutDirectories)
{
	FConfigFile ConfigFile;
	ConfigFile.Read(GetPluginConfigPath());
	ConfigFile.GetArray(ConfigSection, ConfigKey, OutDirectories);
}

void LiveLinkHub::PluginHelpers::WritePluginDirectoriesToConfig(const TArray<FString>& InDirectories)
{
	FConfigFile ConfigFile;
	ConfigFile.Read(GetPluginConfigPath());

	ConfigFile.ResetKeyInSection(ConfigSection, ConfigKey);
	for (const FString& Directory : InDirectories)
	{
		ConfigFile.AddUniqueToSection(ConfigSection, ConfigKey, Directory);
	}

	ConfigFile.Write(GetPluginConfigPath());
}

void LiveLinkHub::PluginHelpers::RestoreSavedPluginDirectories()
{
	TArray<FString> SavedDirectories;
	ReadPluginDirectoriesFromConfig(SavedDirectories);

	bool bPluginPathsChanged = false;

	IPluginManager& PluginManager = IPluginManager::Get();
	for (const FString& Directory : SavedDirectories)
	{
		const bool bRefresh_false = false;
		bPluginPathsChanged |= PluginManager.AddPluginSearchPath(Directory, bRefresh_false);
	}

	if (bPluginPathsChanged)
	{
		PluginManager.RefreshPluginsList();
	}
}

void LiveLinkHub::PluginHelpers::HandlePluginDirectoriesChanged()
{
	IPluginManager& PluginManager = IPluginManager::Get();

	TSet<FExternalPluginPath> PluginSources;
	PluginManager.GetExternalPluginSources(PluginSources);

	TArray<FString> UserDirectories;
	for (const FExternalPluginPath& Path : PluginSources)
	{
		if (Path.Source == EPluginExternalSource::Other)
		{
			UserDirectories.Add(Path.Path);
		}
	}

	WritePluginDirectoriesToConfig(UserDirectories);
}

void LiveLinkHub::PluginHelpers::RegisterPluginDirectoriesChangedHandler()
{
	IPluginBrowser& PluginBrowser = IPluginBrowser::Get();
	PluginBrowser.OnPluginDirectoriesChanged().AddStatic(HandlePluginDirectoriesChanged);
}
