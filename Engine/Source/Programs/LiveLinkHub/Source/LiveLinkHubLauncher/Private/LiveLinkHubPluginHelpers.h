// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/UnrealString.h"


namespace LiveLinkHub::PluginHelpers
{
	/** Returns the path to LiveLinkHubPlugins.ini. */
	FString GetPluginConfigPath();

	/** Returns the user plugin directories stored in LiveLinkHubPlugins.ini. */
	void ReadPluginDirectoriesFromConfig(TArray<FString>& OutDirectories);

	/** Serializes the specified user plugin directories out to LiveLinkHubPlugins.ini. */
	void WritePluginDirectoriesToConfig(const TArray<FString>& InDirectories);

	/** Reads the configured user plugin directories and adds them to the plugin manager search path. */
	void RestoreSavedPluginDirectories();

	/** Stores the latest plugin manager search paths to disk. */
	void HandlePluginDirectoriesChanged();

	/** Connects the `HandlePluginDirectoriesChanged` handler to the plugin browser directories changed event. */
	void RegisterPluginDirectoriesChangedHandler();
}
