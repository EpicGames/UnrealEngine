// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubRun.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformSplash.h"
#include "IPluginBrowser.h"
#include "LaunchEngineLoop.h"
#include "LiveLinkHubPluginHelpers.h"
#include "Modules/ModuleManager.h"
#include "UnrealEdMisc.h"

#ifndef WITH_ASSET_LOADING_AUDIT
#define WITH_ASSET_LOADING_AUDIT 0
#endif


DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHub, Log, All);


int32 RunLiveLinkHub(const TCHAR* CommandLine)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	// Needs to be initialized early for splash / mount points / plugin search paths.
	FCommandLine::Set(CommandLine);

	const FText AppName = NSLOCTEXT("LiveLinkHub", "SplashTextName", "Live Link Hub");
	FPlatformSplash::SetSplashText(SplashTextType::GameName, AppName);

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CommandLine, TEXT("WaitForDebugger")))
	{
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.1f);
		}
		UE_DEBUG_BREAK();
	}
#endif

#if WITH_ASSET_LOADING_AUDIT
	FCoreDelegates::OnSyncLoadPackage.AddLambda([](const FString& PackageName)
		{
			UE_LOGF(LogLiveLinkHub, Display, "Audit: Loaded %ls", *PackageName);
		});
#endif

	bool bPendingRestart = false;
	
	FString DevelopmentProjectPath = TEXT("../../Source/Programs/LiveLinkHub/LiveLinkHub.uproject");
	FString StagedProjectPath = TEXT("../../../LiveLinkHub/LiveLinkHub.uproject");

#if PLATFORM_MAC
	// On Mac the following FileExists calls will fail unless it's an absolute path.
	DevelopmentProjectPath = FPaths::ConvertRelativePathToFull(DevelopmentProjectPath);
	StagedProjectPath = FPaths::ConvertRelativePathToFull(StagedProjectPath);
#endif
	
	if (FPaths::FileExists(*DevelopmentProjectPath))
	{
		// This is a 'in-tree' version of LiveLinkHub, meaning it could be a development build or the version of LLH that is shipped with UnrealEngine.
		FPaths::SetProjectFilePath(*DevelopmentProjectPath);
	}
	else if (FPaths::FileExists(StagedProjectPath))
	{
		// This is build of LiveLinkHub that's distributed on the EGS.
		FPaths::SetProjectFilePath(StagedProjectPath);

		// This is needed to ensure that content generated by the hub will be discoverable (ie. LiveLink recordings)
        const FString GameContentPath = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("LiveLinkHub"), TEXT("Content"));
        FPackageName::RegisterMountPoint(TEXT("/Game/"), GameContentPath);
	}

	// Used by Live Coding, among other things.
	FPlatformMisc::SetUBTTargetName(TEXT("LiveLinkHub"));

	// Plugin directory config save/load
	FCoreDelegates::TSConfigReadyForUse().AddStatic(LiveLinkHub::PluginHelpers::RestoreSavedPluginDirectories);
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda([&bPendingRestart]()
	{
		LiveLinkHub::PluginHelpers::RegisterPluginDirectoriesChangedHandler();
		
		IPluginBrowser& PluginBrowser = IPluginBrowser::Get();
		PluginBrowser.OnRestartClicked().BindLambda([&bPendingRestart]()
		{
			bPendingRestart = true;
			RequestEngineExit(TEXT("Plugin browser requested restart"));
		});
	});

	// Start up the main loop, adding some extra command line arguments:
	const TCHAR* const ExtraArgs = TEXT("-xrtrackingonly");
	int32 Result = GEngineLoop.PreInit(*FString::Printf(TEXT("%s %s %s"),
		*FPaths::GetProjectFilePath(), CommandLine, ExtraArgs));


	// Ensure FEngineLoop::Exit is called for all return paths. (see EngineLoopCleanupGuard)
	ON_SCOPE_EXIT
	{
		GEngineLoop.Exit();

		// Handle this after FEngineLoop::Cleanup, which can take several seconds.
		// We want to defer this as long as possible to avoid/minimize concurrent execution.
		if (bPendingRestart)
		{
			const FString ExeFilename = FUnrealEdMisc::GetProjectEditorBinaryPath();
			FProcHandle Handle = FPlatformProcess::CreateProc(*ExeFilename, FCommandLine::Get(), true, false, false, NULL, 0, NULL, NULL);
			const bool bSuccess = Handle.IsValid();
			if (bSuccess)
			{
				FPlatformProcess::CloseProc(Handle);
			}
			else
			{
				UE_LOGF(LogLiveLinkHub, Error, "Pending restart: FPlatformProcess::CreateProc failed");
			}
		}
	};

	if (Result != 0)
	{
		UE_LOGF(LogLiveLinkHub, Error, "EngineLoop PreInit failed (%i)", Result);
	}

	if (Result != 0 || IsEngineExitRequested())
	{
		return Result;
	}

	{
		Result = GEngineLoop.Init();

		GEditor->ShouldDisableCPUThrottlingDelegates.Add(UEditorEngine::FShouldDisableCPUThrottling::CreateLambda([]() { return true; }));
		GEditor->SetMaxFPS(60);

		// Hide the splash screen now that everything is ready to go
		FPlatformSplash::Hide();

		while (!IsEngineExitRequested())
		{
			GEngineLoop.Tick();
		}
	}

	return Result;
}
