// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include "RequiredProgramMainCPPInclude.h"

#if IS_PROGRAM
IMPLEMENT_APPLICATION(LiveLinkHubLauncher, "LiveLinkHub");
#else
FEngineLoop GEngineLoop;
#endif
