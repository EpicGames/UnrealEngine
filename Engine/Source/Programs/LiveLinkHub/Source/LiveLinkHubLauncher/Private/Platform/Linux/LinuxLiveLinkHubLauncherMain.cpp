// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LiveLinkHubRun.h"
#include "LaunchEngineLoop.h"
#include "UnixCommonStartup.h"

int main(int argc, char *argv[])
{
	return CommonUnixMain(argc, argv, &RunLiveLinkHub, [] { } );
}
