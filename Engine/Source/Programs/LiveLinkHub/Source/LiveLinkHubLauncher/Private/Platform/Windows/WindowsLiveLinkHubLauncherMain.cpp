// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LiveLinkHubRun.h"
#include "HAL/ExceptionHandling.h"
#include "LaunchEngineLoop.h"
#include "Windows/WindowsHWrapper.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceError.h"

#include <shellapi.h>

#if !defined(D3D12_CORE_ENABLED)
	#define D3D12_CORE_ENABLED 0
#endif

#if D3D12_CORE_ENABLED
// Add Agility SDK symbols for Windows
#include "UnrealAgilitySDKLink.inl"
#endif

/**
 * The command-line invocation string, processed using the standard Windows CommandLineToArgvW implementation.
 * This need to be a static global to avoid object unwinding errors in WinMain when SEH is enabled.
 */
static FString GSavedCommandLine;

// Code copied from Engine\Source\Runtime\Launch\Private\Windows\LaunchWindows.cpp
bool ProcessCommandLine()
{
	int argc = 0;
	LPWSTR* argv = ::CommandLineToArgvW(::GetCommandLineW(), &argc);
	if (argv != nullptr)
	{
		// Reconstruct our command line string in a format suitable for consumption by the FParse class
		GSavedCommandLine = "";
		for (int32 Option = 1; Option < argc; Option++)
		{
			GSavedCommandLine += TEXT(" ");

			// Inject quotes in the correct position to preserve arguments containing whitespace
			FString Temp = argv[Option];
			if (Temp.Contains(TEXT(" ")))
			{
				int32 Quote = 0;
				if (Temp.StartsWith(TEXT("-")))
				{
					int32 Separator;
					if (Temp.FindChar('=', Separator))
					{
						Quote = Separator + 1;
					}
				}
				Temp = Temp.Left(Quote) + TEXT("\"") + Temp.Mid(Quote) + TEXT("\"");
			}
			GSavedCommandLine += Temp;
		}

		// Free memory allocated for CommandLineToArgvW() arguments
		::LocalFree(argv);
		return true;
	}

	return false;
}

/**
 * The main application entry point for Windows platforms.
 *
 * @param hInInstance Handle to the current instance of the application.
 * @param hPrevInstance Handle to the previous instance of the application (always NULL).
 * @param lpCmdLine Command line for the application.
 * @param nShowCmd Specifies how the window is to be shown.
 * @return Application's exit value.
 */
int32 WINAPI WinMain(_In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ char* lpCmdLine, _In_ int32 nShowCmd)
{
	hInstance = hInInstance;

	const TCHAR* CmdLine = ::GetCommandLineW();

	// Attempt to process the command-line arguments using the standard Windows implementation
	// (This ensures behavior parity with other platforms where argc and argv are used.)
	if (ProcessCommandLine())
	{
		CmdLine = *GSavedCommandLine;
	}

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CmdLine, TEXT("crashreports")))
	{
		GAlwaysReportCrash = true;
	}
#endif

	int32 ErrorLevel = 0;

#if UE_BUILD_DEBUG
	if (!GAlwaysReportCrash)
#else
	if (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash)
#endif
	{
		ErrorLevel = RunLiveLinkHub(CmdLine);
	}
	else
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
 		{
			GIsGuarded = 1;
			ErrorLevel = RunLiveLinkHub(CmdLine);
			GIsGuarded = 0;
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except (ReportCrash(GetExceptionInformation()))
		{
			ErrorLevel = 1;
			GError->HandleError();
			FPlatformMisc::RequestExit(true);
		}
#endif
	}

	FEngineLoop::AppExit();

	return ErrorLevel;
}
