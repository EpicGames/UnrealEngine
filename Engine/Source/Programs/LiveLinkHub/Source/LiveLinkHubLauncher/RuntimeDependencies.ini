[LiveLinkHubLauncher]
+RuntimeDependencies="/Content/Animation/..."
+RuntimeDependencies="/Content/BasicShapes/..."
+RuntimeDependencies="/Content/Editor/Slate/..."
; Prefix - denotes exclusion
+RuntimeDependencies="-/Content/Editor/Slate/Automation/..."
+RuntimeDependencies="-/Content/Editor/Slate/Testing/..."
+RuntimeDependencies="/Content/BasicShapes/..."
+RuntimeDependencies="/Content/BufferVisualization/..."
+RuntimeDependencies="/Content/EditorMaterials/..."
+RuntimeDependencies="/Content/EditorMeshes/..."
+RuntimeDependencies="/Content/EditorResources/..."
+RuntimeDependencies="/Content/EngineDamageTypes/..."
+RuntimeDependencies="/Content/EngineFonts/..."
+RuntimeDependencies="/Content/EngineDebugMaterials/..."
+RuntimeDependencies="/Content/EngineMaterials/..."
+RuntimeDependencies="/Content/EngineSounds/..."
+RuntimeDependencies="/Content/Functions/..."
+RuntimeDependencies="/Content/Slate/..."

; Individual assets
+RuntimeDependencies="/Content/ArtTools/RenderToTexture/MaterialFunctions/CheckerPattern.uasset"
+RuntimeDependencies="/Content/ArtTools/RenderToTexture/Meshes/S_1_Unit_Plane.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/ColorMaskBrushMaterial_MaskedRegion.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/DataLayer.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/DefaultAlphaTexture.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LandscapeDirtyMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LandscapeGizmoHeight_Mat.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LandscapeGizmo_Mat.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LandscapeGizmo_Mat_Copied.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LandscapeLayerUsageMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/LayerVisMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/MaskBrushMaterial_MaskedRegion.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/MF_ThicknessOutline.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/SelectBrushMaterial_Selected.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/SelectBrushMaterial_SelectedRegion.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/SplineEditorMesh.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/SplineEditorMeshMat.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/ColorMaskBrushMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/MaskBrushMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/SelectBrushMaterial.uasset"
+RuntimeDependencies="/Content/EditorLandscapeResources/WhiteSquareTexture.uasset"
+RuntimeDependencies="/Content/EngineMeshes/Sphere.uasset"
+RuntimeDependencies="/Content/EngineMeshes/M_Mat_UnrealTech.uasset"
+RuntimeDependencies="/Content/EngineResources/Black.uasset"
+RuntimeDependencies="/Content/EngineResources/DefaultTexture.uasset"
+RuntimeDependencies="/Content/EngineResources/DefaultTextureCube.uasset"
+RuntimeDependencies="/Content/EngineResources/DefaultVolumeTexture.uasset"
+RuntimeDependencies="/Content/EngineResources/DefaultVolumeTexture2D.uasset"
+RuntimeDependencies="/Content/EngineResources/GradientTexture0.uasset"
+RuntimeDependencies="/Content/EngineResources/LandscapeVisibilityLayerInfo.uasset"
+RuntimeDependencies="/Content/EngineResources/M_StreamingPause.uasset"
+RuntimeDependencies="/Content/EngineResources/StreamingPauseIcon.uasset"
+RuntimeDependencies="/Content/EngineResources/WhiteSquareTexture.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/CloudGradientTexture.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/CloudWeatherTexture.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_CloudMask.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_CloudPattern.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_NoiseErosion.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_NoiseShape64.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_Profile_08.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_VolumeNoiseErosion32.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/T_VolumeNoiseShape64.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/VT_Lightning.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/VT_PerlinWorley_Balanced.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/m_SimpleVolumetricCloud.uasset"
+RuntimeDependencies="/Content/EngineSky/VolumetricClouds/m_SimpleVolumetricCloud_Inst.uasset"
+RuntimeDependencies="/Content/MapTemplates/Sky/DaylightAmbientCubemap.uasset"
+RuntimeDependencies="/Content/VREditor/LaserPointer/VR_LaserPower_01.uasset"
+RuntimeDependencies="/Content/VREditor/SnapGrid/SnapGridMaterial.uasset"
+RuntimeDependencies="/Content/VREditor/SnapGrid/SnapGridPlaneMesh.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Drop_Item_From_ContentBrowser.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Gizmo_Handle_Clicked.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Object_PickUp.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Object_Snaps_To_Another_Actor.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Object_Snaps_To_Grid.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Scaling_Down.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Scaling_Up.uasset"
+RuntimeDependencies="/Content/VREditor/Sounds/UI/Selection_Changes.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/BoundingBoxCorner.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/BoundingBoxEdge.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/Main.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/PlaneTranslationHandle.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/RotationHandleFull.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/RotationHandleIndicator.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/RotationHandleQuarter.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/SM_Sequencer_Node.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/StartRotationHandleIndicator.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TransformGizmoFreeRotation.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TransformGizmoMaterial.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TransformGizmoMaterial_Inst.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TranslateArrowHandle.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TranslucentTransformGizmoMaterial.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/TranslucentTransformGizmoMaterial_Inst.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/UniformScaleHandle.uasset"
+RuntimeDependencies="/Content/VREditor/TransformGizmo/Xray.uasset"
+RuntimeDependencies="/Content/VREditor/ViewportInteractionAssetContainerData.uasset"

; Plugin content
+RuntimeDependencies="/Plugins/Animation/ACLPlugin/Content/..."
+RuntimeDependencies="/Plugins/Animation/LiveLink/Content/..."
+RuntimeDependencies="/Plugins/Animation/AudioDrivenAnimation/StreamingADA/Content/..."
+RuntimeDependencies="/Plugins/Media/MediaFrameworkUtilities/Content/..."
+RuntimeDependencies="/Plugins/MetaHuman/MetaHumanCoreTechLib/Content/..."
+RuntimeDependencies="/Plugins/MovieScene/LevelSequenceEditor/Content/..."
+RuntimeDependencies="/Plugins/Runtime/AR/ARUtilities/Content/..."
+RuntimeDependencies="/Plugins/Runtime/AR/AppleAR/AppleARKit/Content/..."
+RuntimeDependencies="/Plugins/VirtualProduction/CameraCalibrationCore/Content/..."
+RuntimeDependencies="/Plugins/VirtualProduction/Takes/Content/..."
